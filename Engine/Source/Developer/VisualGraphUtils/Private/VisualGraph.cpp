// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualGraph.h"

///////////////////////////////////////////////////////////////////////////////////
/// FVisualGraphSubGraph
///////////////////////////////////////////////////////////////////////////////////

FString FVisualGraphSubGraph::DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const
{
	FString EdgeContent;

	if(!Nodes.IsEmpty())
	{
		for(const FVisualGraphEdge& Edge : InGraph->GetEdges())
		{
			if(Nodes.Contains(Edge.GetSourceNode()) &&
				Nodes.Contains(Edge.GetTargetNode()))
			{
				EdgeContent += Edge.DumpDot(InGraph, InIndendation + 1);
			}
		}
	}

	FString SubGraphContent;
	for(const FVisualGraphSubGraph& SubGraph : InGraph->GetSubGraphs())
	{
		if(SubGraph.GetParentGraphIndex() == GetIndex())
		{
			SubGraphContent += SubGraph.DumpDot(InGraph, InIndendation + 1);
		}
	}

	if(EdgeContent.IsEmpty() && SubGraphContent.IsEmpty())
	{
		return FString();
	}

	const FString Indentation1 = DumpDotIndentation(InIndendation);
	const FString Indentation2 = DumpDotIndentation(InIndendation + 1);
	const FString ColorContent = DumpDotColor(GetColor());
	const FString StyleContent = DumpDotStyle(GetStyle());

	FString AttributeContent;
	if(DisplayName.IsSet())
	{
		AttributeContent += FString::Printf(TEXT("%slabel = \"%s\";\n"),
			*Indentation2, *DisplayName.GetValue().ToString());
	}
	if(!ColorContent.IsEmpty())
	{
		AttributeContent += FString::Printf(TEXT("%scolor = %s;\n"),
			*Indentation2, *ColorContent);
	}
	if(!StyleContent.IsEmpty())
	{
		AttributeContent += FString::Printf(TEXT("%sstyle = %s;\n"),
			*Indentation2, *StyleContent);
	}

	return FString::Printf(TEXT("%ssubgraph cluster_%s {\n%s%s%s%s}\n"),
		*Indentation1, *GetName().ToString(),
		*AttributeContent,
		*EdgeContent,
		*SubGraphContent,
		*Indentation1
		);
}

///////////////////////////////////////////////////////////////////////////////////
/// FVisualGraph
///////////////////////////////////////////////////////////////////////////////////

FVisualGraph::FVisualGraph(const FName& InName, const FName& InDisplayName)
{
	Name = InName;
	DisplayName = InDisplayName;
	Style = EVisualGraphStyle::Filled;
}

int32 FVisualGraph::AddNode(const FName& InName, TOptional<FName> InDisplayName, TOptional<FLinearColor> InColor,
	TOptional<EVisualGraphShape> InShape, TOptional<EVisualGraphStyle> InStyle)
{
	RefreshNameMapIfNeeded(Nodes, NodeNameMap);
	
	const FName UniqueName = GetUniqueName(InName, NodeNameMap);
	FVisualGraphNode Node;
	Node.Name = UniqueName;
	Node.DisplayName = InDisplayName;
	Node.Color = InColor;
	Node.Shape = InShape;
	Node.Style = InStyle;
	Node.SubGraphIndex = INDEX_NONE;

	return AddElement(Node, Nodes, NodeNameMap);
}

int32 FVisualGraph::AddEdge(int32 InSourceNode, int32 InTargetNode, EVisualGraphEdgeDirection InDirection,
	const FName& InName, TOptional<FName> InDisplayName, TOptional<FLinearColor> InColor,
	TOptional<EVisualGraphStyle> InStyle)
{
	RefreshNameMapIfNeeded(Edges, EdgeNameMap);

	FName DesiredName = InName;
	if(DesiredName.IsNone())
	{
		DesiredName = TEXT("Edge");
	}
	
	const FName UniqueName = GetUniqueName(DesiredName, EdgeNameMap);
	FVisualGraphEdge Edge;
	Edge.Name = UniqueName;
	Edge.DisplayName = InDisplayName;
	Edge.Color = InColor;
	Edge.Style = InStyle;
	Edge.SourceNode = InSourceNode;
	Edge.TargetNode = InTargetNode;
	Edge.Direction = InDirection;

	return AddElement(Edge, Edges, EdgeNameMap);
}

int32 FVisualGraph::AddSubGraph(const FName& InName, TOptional<FName> InDisplayName, int32 InParentGraphIndex,
	TOptional<FLinearColor> InColor, TOptional<EVisualGraphStyle> InStyle, const TArray<int32> InNodes)
{
	RefreshNameMapIfNeeded(SubGraphs, SubGraphNameMap);

	const FName UniqueName = GetUniqueName(InName, SubGraphNameMap);
	FVisualGraphSubGraph SubGraph;
	SubGraph.Name = UniqueName;
	SubGraph.DisplayName = InDisplayName;
	SubGraph.Color = InColor;
	SubGraph.Style = InStyle;
	SubGraph.ParentGraphIndex = InParentGraphIndex;

	const int32 SubGraphIndex = AddElement(SubGraph, SubGraphs, SubGraphNameMap);

	for(const int32 Node : InNodes)
	{
		AddNodeToSubGraph(Node, SubGraphIndex);
	}

	return SubGraphIndex;
}

int32 FVisualGraph::FindNode(const FName& InName) const
{
	if(const int32* IndexPtr= NodeNameMap.Find(InName))
	{
		return *IndexPtr;
	}
	return INDEX_NONE;
}

int32 FVisualGraph::FindEdge(const FName& InName) const
{
	if(const int32* IndexPtr= EdgeNameMap.Find(InName))
	{
		return *IndexPtr;
	}
	return INDEX_NONE;
}

int32 FVisualGraph::FindSubGraph(const FName& InName) const
{
	if(const int32* IndexPtr= SubGraphNameMap.Find(InName))
	{
		return *IndexPtr;
	}
	return INDEX_NONE;
}

bool FVisualGraph::AddNodeToSubGraph(int32 InNodeIndex, int32 InSubGraphIndex)
{
	if(!Nodes.IsValidIndex(InNodeIndex))
	{
		return false;
	}

	if(!SubGraphs.IsValidIndex(InSubGraphIndex))
	{
		return false;
	}

	RemoveNodeFromSubGraph(InNodeIndex);
	SubGraphs[InSubGraphIndex].Nodes.Add(InNodeIndex);
	Nodes[InNodeIndex].SubGraphIndex = InSubGraphIndex;
	return true;
}

bool FVisualGraph::RemoveNodeFromSubGraph(int32 InNodeIndex)
{
	if(!Nodes.IsValidIndex(InNodeIndex))
	{
		return false;
	}
	if(Nodes[InNodeIndex].SubGraphIndex == INDEX_NONE)
	{
		return false;
	}
	if(!SubGraphs.IsValidIndex(Nodes[InNodeIndex].SubGraphIndex))
	{
		return false;
	}

	SubGraphs[Nodes[InNodeIndex].SubGraphIndex].Nodes.Remove(InNodeIndex);
	Nodes[InNodeIndex].SubGraphIndex = INDEX_NONE;
	return true;
}

FString FVisualGraph::DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const
{
	check(InGraph == this);

	const FString Indentation = DumpDotIndentation(InIndendation);
	const FString Indentation2 = DumpDotIndentation(InIndendation + 1);

	const FString ColorContent = DumpDotColor(GetColor());
	const FString StyleContent = DumpDotStyle(GetStyle());
	FString AttributeContent;
	if(!ColorContent.IsEmpty())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("color = %s"), *ColorContent);
	}
	if(!StyleContent.IsEmpty())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("style = %s"), *StyleContent);
	}
	if(!AttributeContent.IsEmpty())
	{
		AttributeContent = FString::Printf(TEXT("%snode [ %s ];\n"), *Indentation2, *AttributeContent);
	}

	FString SubGraphContent;
	for(const FVisualGraphSubGraph& SubGraph : SubGraphs)
	{
		if(SubGraph.GetParentGraphIndex() == INDEX_NONE)
		{
			SubGraphContent += SubGraph.DumpDot(this, InIndendation + 1);
		}
	}

	bool bIsDirectedGraph = true;
	FString EdgeContent;
	for(const FVisualGraphEdge& Edge : Edges)
	{
		if(Edge.GetDirection() == EVisualGraphEdgeDirection::BothWays)
		{
			bIsDirectedGraph = false;
		}

		const FVisualGraphNode& SourceNode = Nodes[Edge.GetSourceNode()];
		const FVisualGraphNode& TargetNode = Nodes[Edge.GetTargetNode()];

		// if both source and target node are within the same subgraph
		if((SourceNode.SubGraphIndex == TargetNode.SubGraphIndex) &&
			(SourceNode.SubGraphIndex != INDEX_NONE))
		{
			continue;
		}
		
		EdgeContent += Edge.DumpDot(this, InIndendation + 1);
	}

	FString NodeContent;
	for(const FVisualGraphNode& Node : Nodes)
	{
		NodeContent += Node.DumpDot(this, InIndendation + 1);
	}

	const FString GraphToken = bIsDirectedGraph ? TEXT("digraph") : TEXT("graph"); 
	return FString::Printf(TEXT("%s%s %s {\n%srankdir=\"LR\";\n%s%s%s%s%s}\n"),
		*Indentation, *GraphToken, *GetName().ToString(),
		*Indentation2,
		*AttributeContent,
		*SubGraphContent,
		*EdgeContent,
		*NodeContent,
		*Indentation
		);
}

bool FVisualGraph::IsNameAvailable(const FName& InName, const TMap<FName, int32>& InMap)
{
	return !InMap.Contains(InName);
}

FName FVisualGraph::GetUniqueName(const FName& InName, const TMap<FName, int32>& InMap)
{
	const FName DesiredName = InName;
	FName Name = DesiredName;
	int32 Suffix = 0;
	while (!IsNameAvailable(Name, InMap))
	{
		Suffix++;
		Name = *FString::Printf(TEXT("%s_%d"), *DesiredName.ToString(), Suffix);
	}
	return Name;
}

void FVisualGraph::TransitiveReduction(TFunction<bool(FVisualGraphEdge&)> KeepEdgeFunction)
{
	// mark up all valid edges
	TArray<int32> ValidEdge;
	ValidEdge.AddUninitialized(Nodes.Num() * Nodes.Num());
	for(int32 EdgeIndex = 0; EdgeIndex < ValidEdge.Num(); EdgeIndex++)
	{
		ValidEdge[EdgeIndex] = -1;
	}
	for(const FVisualGraphEdge& Edge : Edges)
	{
		const int32 EdgeIndex = Edge.GetSourceNode() * Nodes.Num() + Edge.GetTargetNode();
		ValidEdge[EdgeIndex] = Edge.GetIndex();
	}


	TArray<int32> EdgesToRemove;

	// perform transitive reduction on the graph
	for(const FVisualGraphNode& NodeA : Nodes)
	{
		for(const FVisualGraphNode& NodeB : Nodes)
		{
			for(const FVisualGraphNode& NodeC : Nodes)
			{
				const int32 EdgeIndexAB = NodeA.GetIndex() * Nodes.Num() + NodeB.GetIndex();
				const int32 EdgeIndexBC = NodeB.GetIndex() * Nodes.Num() + NodeC.GetIndex();
				const int32 EdgeIndexAC = NodeA.GetIndex() * Nodes.Num() + NodeC.GetIndex();

				if(ValidEdge[EdgeIndexAC] != INDEX_NONE &&
					ValidEdge[EdgeIndexAB] != INDEX_NONE &&
					ValidEdge[EdgeIndexBC] != INDEX_NONE)
				{
					const int32 EdgeIndex = ValidEdge[EdgeIndexAC];
					if(!KeepEdgeFunction(Edges[EdgeIndex]))
					{
						EdgesToRemove.AddUnique(EdgeIndex);
					}
					ValidEdge[EdgeIndexAC] = INDEX_NONE;
				}
			}
		}
	}

	Algo::Sort(EdgesToRemove);
	Algo::Reverse(EdgesToRemove);

	for(const int32 EdgeToRemove : EdgesToRemove)
	{
		Edges.RemoveAt(EdgeToRemove);
	}
}