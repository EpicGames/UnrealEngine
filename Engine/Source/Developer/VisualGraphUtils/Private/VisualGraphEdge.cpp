// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualGraphEdge.h"
#include "VisualGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualGraphEdge)

///////////////////////////////////////////////////////////////////////////////////
/// FVisualGraphEdge
///////////////////////////////////////////////////////////////////////////////////

FString FVisualGraphEdge::DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const
{
	const FString Indentation = DumpDotIndentation(InIndendation);
	const FString ColorContent = DumpDotColor(GetColor());
	const FString StyleContent = DumpDotStyle(GetStyle());

	static const FString DigraphConnector = TEXT("->");
	static const FString GraphConnector = TEXT("--");
	const FString& ConnectorContent = (GetDirection() == EVisualGraphEdgeDirection::BothWays) ? GraphConnector : DigraphConnector; 

	FString SourceNodeName = InGraph->GetNodes()[GetSourceNode()].GetName().ToString();
	FString TargetNodeName = InGraph->GetNodes()[GetTargetNode()].GetName().ToString();

	if(GetDirection() == EVisualGraphEdgeDirection::SourceToTarget)
	{
		Swap(SourceNodeName, TargetNodeName);
	}

	FString AttributeContent;
	if(DisplayName.IsSet())
	{
		AttributeContent += FString::Printf(TEXT("label = \"%s\""), *DisplayName.GetValue().ToString());
	}
	if(Tooltip.IsSet())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("tooltip = \"%s\""), *Tooltip.GetValue());
	}
	if(!ColorContent.IsEmpty())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("color = %s"), *ColorContent);
	}
	if(!StyleContent.IsEmpty())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("style = %s"), *StyleContent);
	}

	if(!AttributeContent.IsEmpty())
	{
		AttributeContent = FString::Printf(TEXT(" [ %s ]"), *AttributeContent);
	}

	return FString::Printf(TEXT("%s%s %s %s%s;\n"),
		*Indentation, *SourceNodeName, *ConnectorContent, *TargetNodeName, *AttributeContent);
}
