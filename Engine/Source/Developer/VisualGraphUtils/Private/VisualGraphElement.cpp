// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualGraphElement.h"
#include "VisualGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualGraphElement)

///////////////////////////////////////////////////////////////////////////////////
/// FVisualGraphElement
///////////////////////////////////////////////////////////////////////////////////

FString FVisualGraphElement::DumpDotIndentation(int32 InIndentation)
{
	check(InIndentation >= 0);
	
	static TArray<FString> Indentations;
	while(Indentations.Num() <= InIndentation)
	{
		const int32 Indentation = Indentations.Num();
		if(Indentation == 0)
		{
			Indentations.Add(FString());
		}
		else
		{
			Indentations.Add(Indentations[Indentation - 1] + TEXT("  "));
		}
	}

	return Indentations[InIndentation];
}

FString FVisualGraphElement::DumpDotColor(const TOptional<FLinearColor>& InColor)
{
	if(!InColor.IsSet())
	{
		return FString();
	}

	const FLinearColor Color = InColor.GetValue();
	
	static TMap<FName, FLinearColor> ColorScheme;
	if(ColorScheme.IsEmpty())
	{
		ColorScheme.Add(TEXT("alice blue"), FColor::FromHex(TEXT("#F0F8FF")));
		ColorScheme.Add(TEXT("antique white"), FColor::FromHex(TEXT("#FAEBD7")));
		//ColorScheme.Add(TEXT("aqua"), FColor::FromHex(TEXT("#00FFFF")));
		ColorScheme.Add(TEXT("aquamarine"), FColor::FromHex(TEXT("#7FFFD4")));
		ColorScheme.Add(TEXT("azure"), FColor::FromHex(TEXT("#F0FFFF")));
		ColorScheme.Add(TEXT("beige"), FColor::FromHex(TEXT("#F5F5DC")));
		ColorScheme.Add(TEXT("bisque"), FColor::FromHex(TEXT("#FFE4C4")));
		ColorScheme.Add(TEXT("black"), FColor::FromHex(TEXT("#000000")));
		ColorScheme.Add(TEXT("blanched almond"), FColor::FromHex(TEXT("#FFEBCD")));
		ColorScheme.Add(TEXT("blue"), FColor::FromHex(TEXT("#0000FF")));
		ColorScheme.Add(TEXT("blue violet"), FColor::FromHex(TEXT("#8A2BE2")));
		ColorScheme.Add(TEXT("brown"), FColor::FromHex(TEXT("#A52A2A")));
		ColorScheme.Add(TEXT("burlywood"), FColor::FromHex(TEXT("#DEB887")));
		ColorScheme.Add(TEXT("cadet blue"), FColor::FromHex(TEXT("#5F9EA0")));
		ColorScheme.Add(TEXT("chartreuse"), FColor::FromHex(TEXT("#7FFF00")));
		ColorScheme.Add(TEXT("chocolate"), FColor::FromHex(TEXT("#D2691E")));
		ColorScheme.Add(TEXT("coral"), FColor::FromHex(TEXT("#FF7F50")));
		ColorScheme.Add(TEXT("cornflower blue"), FColor::FromHex(TEXT("#6495ED")));
		ColorScheme.Add(TEXT("cornsilk"), FColor::FromHex(TEXT("#FFF8DC")));
		ColorScheme.Add(TEXT("crimson"), FColor::FromHex(TEXT("#DC143C")));
		ColorScheme.Add(TEXT("cyan"), FColor::FromHex(TEXT("#00FFFF")));
		ColorScheme.Add(TEXT("dark blue"), FColor::FromHex(TEXT("#00008B")));
		ColorScheme.Add(TEXT("dark cyan"), FColor::FromHex(TEXT("#008B8B")));
		ColorScheme.Add(TEXT("dark goldenrod"), FColor::FromHex(TEXT("#B8860B")));
		ColorScheme.Add(TEXT("dark gray"), FColor::FromHex(TEXT("#A9A9A9")));
		ColorScheme.Add(TEXT("dark green"), FColor::FromHex(TEXT("#006400")));
		ColorScheme.Add(TEXT("dark khaki"), FColor::FromHex(TEXT("#BDB76B")));
		ColorScheme.Add(TEXT("dark magenta"), FColor::FromHex(TEXT("#8B008B")));
		ColorScheme.Add(TEXT("dark olive green"), FColor::FromHex(TEXT("#556B2F")));
		ColorScheme.Add(TEXT("dark orange"), FColor::FromHex(TEXT("#FF8C00")));
		ColorScheme.Add(TEXT("dark orchid"), FColor::FromHex(TEXT("#9932CC")));
		ColorScheme.Add(TEXT("dark red"), FColor::FromHex(TEXT("#8B0000")));
		ColorScheme.Add(TEXT("dark salmon"), FColor::FromHex(TEXT("#E9967A")));
		ColorScheme.Add(TEXT("dark sea green"), FColor::FromHex(TEXT("#8FBC8F")));
		ColorScheme.Add(TEXT("dark slate blue"), FColor::FromHex(TEXT("#483D8B")));
		ColorScheme.Add(TEXT("dark slate gray"), FColor::FromHex(TEXT("#2F4F4F")));
		ColorScheme.Add(TEXT("dark turquoise"), FColor::FromHex(TEXT("#00CED1")));
		ColorScheme.Add(TEXT("dark violet"), FColor::FromHex(TEXT("#9400D3")));
		ColorScheme.Add(TEXT("deep pink"), FColor::FromHex(TEXT("#FF1493")));
		ColorScheme.Add(TEXT("deep sky blue"), FColor::FromHex(TEXT("#00BFFF")));
		ColorScheme.Add(TEXT("dim gray"), FColor::FromHex(TEXT("#696969")));
		ColorScheme.Add(TEXT("dodger blue"), FColor::FromHex(TEXT("#1E90FF")));
		ColorScheme.Add(TEXT("firebrick"), FColor::FromHex(TEXT("#B22222")));
		ColorScheme.Add(TEXT("floral white"), FColor::FromHex(TEXT("#FFFAF0")));
		ColorScheme.Add(TEXT("forest green"), FColor::FromHex(TEXT("#228B22")));
		ColorScheme.Add(TEXT("fuchsia"), FColor::FromHex(TEXT("#FF00FF")));
		ColorScheme.Add(TEXT("gainsboro"), FColor::FromHex(TEXT("#DCDCDC")));
		ColorScheme.Add(TEXT("ghost white"), FColor::FromHex(TEXT("#F8F8FF")));
		ColorScheme.Add(TEXT("gold"), FColor::FromHex(TEXT("#FFD700")));
		ColorScheme.Add(TEXT("goldenrod"), FColor::FromHex(TEXT("#DAA520")));
		ColorScheme.Add(TEXT("gray"), FColor::FromHex(TEXT("#BEBEBE")));
		ColorScheme.Add(TEXT("web gray"), FColor::FromHex(TEXT("#808080")));
		ColorScheme.Add(TEXT("green"), FColor::FromHex(TEXT("#00FF00")));
		ColorScheme.Add(TEXT("web green"), FColor::FromHex(TEXT("#008000")));
		ColorScheme.Add(TEXT("green yellow"), FColor::FromHex(TEXT("#ADFF2F")));
		ColorScheme.Add(TEXT("honeydew"), FColor::FromHex(TEXT("#F0FFF0")));
		ColorScheme.Add(TEXT("hot pink"), FColor::FromHex(TEXT("#FF69B4")));
		ColorScheme.Add(TEXT("indian red"), FColor::FromHex(TEXT("#CD5C5C")));
		ColorScheme.Add(TEXT("indigo"), FColor::FromHex(TEXT("#4B0082")));
		ColorScheme.Add(TEXT("ivory"), FColor::FromHex(TEXT("#FFFFF0")));
		ColorScheme.Add(TEXT("khaki"), FColor::FromHex(TEXT("#F0E68C")));
		ColorScheme.Add(TEXT("lavender"), FColor::FromHex(TEXT("#E6E6FA")));
		ColorScheme.Add(TEXT("lavender blush"), FColor::FromHex(TEXT("#FFF0F5")));
		ColorScheme.Add(TEXT("lawn green"), FColor::FromHex(TEXT("#7CFC00")));
		ColorScheme.Add(TEXT("lemon chiffon"), FColor::FromHex(TEXT("#FFFACD")));
		ColorScheme.Add(TEXT("light blue"), FColor::FromHex(TEXT("#ADD8E6")));
		ColorScheme.Add(TEXT("light coral"), FColor::FromHex(TEXT("#F08080")));
		ColorScheme.Add(TEXT("light cyan"), FColor::FromHex(TEXT("#E0FFFF")));
		ColorScheme.Add(TEXT("light goldenrod"), FColor::FromHex(TEXT("#FAFAD2")));
		ColorScheme.Add(TEXT("light gray"), FColor::FromHex(TEXT("#D3D3D3")));
		ColorScheme.Add(TEXT("light green"), FColor::FromHex(TEXT("#90EE90")));
		ColorScheme.Add(TEXT("light pink"), FColor::FromHex(TEXT("#FFB6C1")));
		ColorScheme.Add(TEXT("light salmon"), FColor::FromHex(TEXT("#FFA07A")));
		ColorScheme.Add(TEXT("light sea green"), FColor::FromHex(TEXT("#20B2AA")));
		ColorScheme.Add(TEXT("light sky blue"), FColor::FromHex(TEXT("#87CEFA")));
		ColorScheme.Add(TEXT("light slate gray"), FColor::FromHex(TEXT("#778899")));
		ColorScheme.Add(TEXT("light steel blue"), FColor::FromHex(TEXT("#B0C4DE")));
		ColorScheme.Add(TEXT("light yellow"), FColor::FromHex(TEXT("#FFFFE0")));
		ColorScheme.Add(TEXT("lime"), FColor::FromHex(TEXT("#00FF00")));
		ColorScheme.Add(TEXT("lime green"), FColor::FromHex(TEXT("#32CD32")));
		ColorScheme.Add(TEXT("linen"), FColor::FromHex(TEXT("#FAF0E6")));
		ColorScheme.Add(TEXT("magenta"), FColor::FromHex(TEXT("#FF00FF")));
		ColorScheme.Add(TEXT("maroon"), FColor::FromHex(TEXT("#B03060")));
		ColorScheme.Add(TEXT("web maroon"), FColor::FromHex(TEXT("#800000")));
		ColorScheme.Add(TEXT("medium aquamarine"), FColor::FromHex(TEXT("#66CDAA")));
		ColorScheme.Add(TEXT("medium blue"), FColor::FromHex(TEXT("#0000CD")));
		ColorScheme.Add(TEXT("medium orchid"), FColor::FromHex(TEXT("#BA55D3")));
		ColorScheme.Add(TEXT("medium purple"), FColor::FromHex(TEXT("#9370DB")));
		ColorScheme.Add(TEXT("medium sea green"), FColor::FromHex(TEXT("#3CB371")));
		ColorScheme.Add(TEXT("medium slate blue"), FColor::FromHex(TEXT("#7B68EE")));
		ColorScheme.Add(TEXT("medium spring green"), FColor::FromHex(TEXT("#00FA9A")));
		ColorScheme.Add(TEXT("medium turquoise"), FColor::FromHex(TEXT("#48D1CC")));
		ColorScheme.Add(TEXT("medium violet red"), FColor::FromHex(TEXT("#C71585")));
		ColorScheme.Add(TEXT("midnight blue"), FColor::FromHex(TEXT("#191970")));
		ColorScheme.Add(TEXT("mint cream"), FColor::FromHex(TEXT("#F5FFFA")));
		ColorScheme.Add(TEXT("misty rose"), FColor::FromHex(TEXT("#FFE4E1")));
		ColorScheme.Add(TEXT("moccasin"), FColor::FromHex(TEXT("#FFE4B5")));
		ColorScheme.Add(TEXT("navajo white"), FColor::FromHex(TEXT("#FFDEAD")));
		ColorScheme.Add(TEXT("navy blue"), FColor::FromHex(TEXT("#000080")));
		ColorScheme.Add(TEXT("old lace"), FColor::FromHex(TEXT("#FDF5E6")));
		ColorScheme.Add(TEXT("olive"), FColor::FromHex(TEXT("#808000")));
		ColorScheme.Add(TEXT("olive drab"), FColor::FromHex(TEXT("#6B8E23")));
		ColorScheme.Add(TEXT("orange"), FColor::FromHex(TEXT("#FFA500")));
		ColorScheme.Add(TEXT("orange red"), FColor::FromHex(TEXT("#FF4500")));
		ColorScheme.Add(TEXT("orchid"), FColor::FromHex(TEXT("#DA70D6")));
		ColorScheme.Add(TEXT("pale goldenrod"), FColor::FromHex(TEXT("#EEE8AA")));
		ColorScheme.Add(TEXT("pale green"), FColor::FromHex(TEXT("#98FB98")));
		ColorScheme.Add(TEXT("pale turquoise"), FColor::FromHex(TEXT("#AFEEEE")));
		ColorScheme.Add(TEXT("pale violet red"), FColor::FromHex(TEXT("#DB7093")));
		ColorScheme.Add(TEXT("papaya whip"), FColor::FromHex(TEXT("#FFEFD5")));
		ColorScheme.Add(TEXT("peach puff"), FColor::FromHex(TEXT("#FFDAB9")));
		ColorScheme.Add(TEXT("peru"), FColor::FromHex(TEXT("#CD853F")));
		ColorScheme.Add(TEXT("pink"), FColor::FromHex(TEXT("#FFC0CB")));
		ColorScheme.Add(TEXT("plum"), FColor::FromHex(TEXT("#DDA0DD")));
		ColorScheme.Add(TEXT("powder blue"), FColor::FromHex(TEXT("#B0E0E6")));
		ColorScheme.Add(TEXT("purple"), FColor::FromHex(TEXT("#A020F0")));
		ColorScheme.Add(TEXT("web purple"), FColor::FromHex(TEXT("#800080")));
		ColorScheme.Add(TEXT("rebecca purple"), FColor::FromHex(TEXT("#663399")));
		ColorScheme.Add(TEXT("red"), FColor::FromHex(TEXT("#FF0000")));
		ColorScheme.Add(TEXT("rosy brown"), FColor::FromHex(TEXT("#BC8F8F")));
		ColorScheme.Add(TEXT("royal blue"), FColor::FromHex(TEXT("#4169E1")));
		ColorScheme.Add(TEXT("saddle brown"), FColor::FromHex(TEXT("#8B4513")));
		ColorScheme.Add(TEXT("salmon"), FColor::FromHex(TEXT("#FA8072")));
		ColorScheme.Add(TEXT("sandy brown"), FColor::FromHex(TEXT("#F4A460")));
		ColorScheme.Add(TEXT("sea green"), FColor::FromHex(TEXT("#2E8B57")));
		ColorScheme.Add(TEXT("seashell"), FColor::FromHex(TEXT("#FFF5EE")));
		ColorScheme.Add(TEXT("sienna"), FColor::FromHex(TEXT("#A0522D")));
		ColorScheme.Add(TEXT("silver"), FColor::FromHex(TEXT("#C0C0C0")));
		ColorScheme.Add(TEXT("sky blue"), FColor::FromHex(TEXT("#87CEEB")));
		ColorScheme.Add(TEXT("slate blue"), FColor::FromHex(TEXT("#6A5ACD")));
		ColorScheme.Add(TEXT("slate gray"), FColor::FromHex(TEXT("#708090")));
		ColorScheme.Add(TEXT("snow"), FColor::FromHex(TEXT("#FFFAFA")));
		ColorScheme.Add(TEXT("spring green"), FColor::FromHex(TEXT("#00FF7F")));
		ColorScheme.Add(TEXT("steel blue"), FColor::FromHex(TEXT("#4682B4")));
		ColorScheme.Add(TEXT("tan"), FColor::FromHex(TEXT("#D2B48C")));
		ColorScheme.Add(TEXT("teal"), FColor::FromHex(TEXT("#008080")));
		ColorScheme.Add(TEXT("thistle"), FColor::FromHex(TEXT("#D8BFD8")));
		ColorScheme.Add(TEXT("tomato"), FColor::FromHex(TEXT("#FF6347")));
		ColorScheme.Add(TEXT("turquoise"), FColor::FromHex(TEXT("#40E0D0")));
		ColorScheme.Add(TEXT("violet"), FColor::FromHex(TEXT("#EE82EE")));
		ColorScheme.Add(TEXT("wheat"), FColor::FromHex(TEXT("#F5DEB3")));
		ColorScheme.Add(TEXT("white"), FColor::FromHex(TEXT("#FFFFFF")));
		ColorScheme.Add(TEXT("white smoke"), FColor::FromHex(TEXT("#F5F5F5")));
		ColorScheme.Add(TEXT("yellow"), FColor::FromHex(TEXT("#FFFF00")));
		ColorScheme.Add(TEXT("yellow green"), FColor::FromHex(TEXT("#9ACD32")));
	}

	FName SchemeName = NAME_None;
	float MinDelta = FLT_MAX;
	for(const TPair<FName, FLinearColor>& Pair : ColorScheme)
	{
		const float Delta = FLinearColor::Dist(Pair.Value, Color);
		if(MinDelta > Delta)
		{
			MinDelta = Delta;
			SchemeName = Pair.Key;
		}
	}

	FString SchemeString = SchemeName.ToString();
	SchemeString.RemoveSpacesInline();
	return SchemeString;
}

FString FVisualGraphElement::DumpDotShape(const TOptional<EVisualGraphShape>& InShape)
{
	if(!InShape.IsSet())
	{
		return FString();
	}
	return StaticEnum<EVisualGraphShape>()->GetDisplayNameTextByValue((int64)InShape.GetValue()).ToString().ToLower();
}

FString FVisualGraphElement::DumpDotStyle(const TOptional<EVisualGraphStyle>& InStyle)
{
	if(!InStyle.IsSet())
	{
		return FString();
	}
	return StaticEnum<EVisualGraphStyle>()->GetDisplayNameTextByValue((int64)InStyle.GetValue()).ToString().ToLower();
}
