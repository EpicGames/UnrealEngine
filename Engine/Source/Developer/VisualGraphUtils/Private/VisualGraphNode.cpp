// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualGraphNode.h"
#include "VisualGraph.h"

///////////////////////////////////////////////////////////////////////////////////
/// FVisualGraphNode
///////////////////////////////////////////////////////////////////////////////////

FString FVisualGraphNode::DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const
{
	const FString Indentation = DumpDotIndentation(InIndendation);
	const FString StyleContent = DumpDotStyle(GetStyle());
	const FString ShapeContent = DumpDotShape(GetShape());
	const FString ColorContent = DumpDotColor(GetColor());

	FString AttributeContent;
	if(DisplayName.IsSet())
	{
		AttributeContent += FString::Printf(TEXT("label = \"%s\""), *DisplayName.GetValue().ToString());
	}
	if(Tooltip.IsSet())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("tooltip = \"%s\""), *Tooltip.GetValue());
	}
	if(!StyleContent.IsEmpty())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("style = %s"), *StyleContent);
	}
	if(!ShapeContent.IsEmpty())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("shape = %s"), *ShapeContent);
	}
	if(!ColorContent.IsEmpty())
	{
		if(!AttributeContent.IsEmpty())
		{
			AttributeContent += TEXT(", ");
		}
		AttributeContent += FString::Printf(TEXT("color = %s"), *ColorContent);

		if(GetColor().IsSet())
		{
			if(!GetStyle().IsSet() || GetStyle().GetValue() == EVisualGraphStyle::Filled)
			{
				const FLinearColor CurrentColor = GetColor().GetValue();
				const float Brightness = (0.299f * CurrentColor.R + 0.587f * CurrentColor.G + 0.114f * CurrentColor.B);
				if(Brightness < 0.5f)
				{
					AttributeContent += TEXT(", fontcolor = white");
				}
			}
		}
	}

	if(!AttributeContent.IsEmpty())
	{
		AttributeContent = FString::Printf(TEXT(" [ %s ]"), *AttributeContent);
	}

	return FString::Printf(TEXT("%s%s%s;\n"), *Indentation, *GetName().ToString(), *AttributeContent);
}

