// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualGraphObjectUtils.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "VisualGraphUtilsModule.h"
#include "Elements/Framework/TypedElementListObjectUtil.h"
#include "Serialization/ArchiveUObject.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR

#include "HAL/PlatformApplicationMisc.h"

///////////////////////////////////////////////////////////////////////////////////
/// Console Commands
///////////////////////////////////////////////////////////////////////////////////

FAutoConsoleCommandWithWorldAndArgs FCmdVisualGraphUtilsLogClassNames
(
	TEXT("VisualGraphUtils.Object.LogClassNames"),
	TEXT("Logs all class path names given a partial name"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* InWorld)
	{
		if(InParams.Num() == 0)
		{
			UE_LOGF(LogVisualGraphUtils, Error, "Unsufficient parameters. Command usage:");
			UE_LOGF(LogVisualGraphUtils, Error, "Example: VisualGraphUtils.Object.LogClassNames skeletal material");
			UE_LOGF(LogVisualGraphUtils, Error, "Provide one or more search tokens for case insensitive search within all known (loaded) class names.");
			return;
		}

		FString ClipboardContent;

		for(const FString& SearchToken : InParams)
		{
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				if(ClassIt->GetName().Contains(SearchToken, ESearchCase::IgnoreCase))
				{
					const FString PathName = ClassIt->GetPathName();
					ClipboardContent += PathName + TEXT("\n");
					UE_LOGF(LogVisualGraphUtils, Display, "Found Class %ls", *PathName);
				}
			}
		}

		FPlatformApplicationMisc::ClipboardCopy(*ClipboardContent);

		UE_LOGF(LogVisualGraphUtils, Display, "The content has also been copied to the clipboard.");
	})
);

FAutoConsoleCommandWithWorldAndArgs FCmdVisualGraphUtilsLogInstancesOfClass
(
	TEXT("VisualGraphUtils.Object.LogInstancesOfClass"),
	TEXT("Logs all instances of a given class"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* InWorld)
	{
		if(InParams.Num() == 0)
		{
			UE_LOGF(LogVisualGraphUtils, Error, "Unsufficient parameters. Command usage:");
			UE_LOGF(LogVisualGraphUtils, Error, "Example: VisualGraphUtils.Object.LogInstancesOfClass /Script/Engine.EdGraphNode");
			UE_LOGF(LogVisualGraphUtils, Error, "Provide one or more class path names");
			return;
		}

		FString ClipboardContent;
		for(const FString& ObjectPathName : InParams)
		{
			if(UClass* Class = UClass::TryFindTypeSlow<UClass>(ObjectPathName))
			{
				ForEachObjectOfClass(Class, [&ClipboardContent](UObject* ObjectOfClass)
				{
					const FString PathName = ObjectOfClass->GetPathName();
					ClipboardContent += PathName + TEXT("\n");
					UE_LOGF(LogVisualGraphUtils, Display, "Found Instance %ls", *PathName);
				}, true, RF_NoFlags);
			}
			else
			{
				UE_LOGF(LogVisualGraphUtils, Error, "Class with pathname '%ls' not found.", *ObjectPathName);
				return;
			}
		}

		FPlatformApplicationMisc::ClipboardCopy(*ClipboardContent);

		UE_LOGF(LogVisualGraphUtils, Display, "The content has also been copied to the clipboard.");
	})
);

FAutoConsoleCommandWithWorldAndArgs FCmdVisualGraphUtilsCollectReferences
(
	TEXT("VisualGraphUtils.Object.CollectReferences"),
	TEXT("Traces all references of an object"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* InWorld)
	{
		if(InParams.Num() == 0)
		{
			UE_LOGF(LogVisualGraphUtils, Error, "Unsufficient parameters. Command usage:");
			UE_LOGF(LogVisualGraphUtils, Error, "Example: VisualGraphUtils.Object.CollectReferences /Game/Animation/ControlRig/BasicControls_CtrlRig skip=/Script/RigVMDeveloper.RigVMNode skip=/Script/RigVMDeveloper.RigVMLink skip=/Script/Engine.EdGraph skip=/Script/Engine.EdGraphNode withinpackageonly");
			UE_LOGF(LogVisualGraphUtils, Error, "Provide one or more path names for packages or objects to collect all references.");
			UE_LOGF(LogVisualGraphUtils, Error, "Optionally provide one or more path names for classes / outers to skip using the skip= token.");
			UE_LOGF(LogVisualGraphUtils, Error, "Use the the withinpackageonly token to limit the search within the source package(s) only.");
			UE_LOGF(LogVisualGraphUtils, Error, "Use the the skipchildren token to skip following objects within each outer (children).");
			UE_LOGF(LogVisualGraphUtils, Error, "Use the the skipreferences token to skip following direct references.");
			return;
		}

		bool bTraverseOuters = true;
		bool bCollectReferences = true;
		bool bWithinPackageOnly = false;

		TArray<FString> ObjectPathNames;
		TArray<FString> SkipPathNames;
		for(const FString& InParam : InParams)
		{
			static const FString ObjectPathToken = TEXT("path"); 
			static const FString SkipToken = TEXT("skip");
			static const FString SkipChildrenToken = TEXT("skipchildren");
			static const FString SkipReferencesToken = TEXT("skipreferences");
			static const FString WithinPackageOnlyToken = TEXT("withinpackageonly");
			FString Token = ObjectPathToken;
			FString Content = InParam;

			if(InParam.Contains(TEXT("=")))
			{
				const int32 Pos = InParam.Find(TEXT("="));
				Token = InParam.Left(Pos);
				Content = InParam.Mid(Pos+1);
				Token.TrimStartAndEndInline();
				Token.ToLowerInline();
				Content.TrimStartAndEndInline();
			}
			else if(InParam.ToLower() == SkipChildrenToken)
			{
				bTraverseOuters = false;
				continue;
			}
			else if(InParam.ToLower() == SkipReferencesToken)
			{
				bCollectReferences = false;
				continue;
			}
			else if(InParam.ToLower() == WithinPackageOnlyToken)
			{
				bWithinPackageOnly = true;
				continue;
			}

			if(Token == ObjectPathToken)
			{
				ObjectPathNames.Add(Content);
			}
			else if(Token == SkipToken)
			{
				SkipPathNames.Add(Content);
			}
		}

		TArray<UObject*> Objects;
		TArray<UObject*> OutersToUse;
		for(const FString& ObjectPathName : ObjectPathNames)
		{
			UE_CLOGF(FPackageName::IsShortPackageName(ObjectPathName), LogVisualGraphUtils, Warning, "Expected path name but got short name: \"%ls\"", *ObjectPathName);
			if(UObject* Object = FindObject<UObject>(nullptr, *ObjectPathName, EFindObjectFlags::None))
			{
				Objects.Add(Object);

				if(bWithinPackageOnly)
				{
					OutersToUse.AddUnique(Object->GetOutermost());
				}
			}
			else
			{
				UE_LOGF(LogVisualGraphUtils, Error, "Object with pathname '%ls' not found.", *ObjectPathName);
				return;
			}
		}

		TArray<UClass*> ClassesToSkip;
		TArray<UObject*> OutersToSkip;
		for(const FString& SkipPathName : SkipPathNames)
		{
			UE_CLOGF(FPackageName::IsShortPackageName(SkipPathName), LogVisualGraphUtils, Warning, "Expected path name but got short name: \"%ls\"", *SkipPathName);
			if(UObject* ObjectToSkip = FindObject<UObject>(nullptr, *SkipPathName, EFindObjectFlags::None))
			{
				if(UClass* ClassToSkip = Cast<UClass>(ObjectToSkip))
				{
					ClassesToSkip.Add(ClassToSkip);
				}
				else
				{
					OutersToSkip.Add(ObjectToSkip);
				}
			}
			else
			{
				UE_LOGF(LogVisualGraphUtils, Error, "Object with pathname '%ls' not found.", *SkipPathName);
				return;
			}
		}

		const FString DotGraphContent = FVisualGraphObjectUtils::TraverseUObjectReferences(
			Objects, ClassesToSkip, OutersToSkip, OutersToUse,
			bTraverseOuters, bCollectReferences).DumpDot();
		FPlatformApplicationMisc::ClipboardCopy(*DotGraphContent);

		UE_LOGF(LogVisualGraphUtils, Display, "The content has been copied to the clipboard.");
	})
);

FAutoConsoleCommandWithWorldAndArgs FCmdVisualGraphUtilsCollectTickables
(
	TEXT("VisualGraphUtils.Object.CollectTickables"),
	TEXT("Traces all tickables of an object"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* InWorld)
	{
		if(InParams.Num() == 0)
		{
			UE_LOGF(LogVisualGraphUtils, Error, "Unsufficient parameters. Command usage:");
			UE_LOGF(LogVisualGraphUtils, Error, "Example: VisualGraphUtils.Object.CollectTickables /Game/Animation/ControlRig/BasicControls_CtrlRig skip=/Script/RigVMDeveloper.RigVMNode skip=/Script/RigVMDeveloper.RigVMLink skip=/Script/Engine.EdGraph skip=/Script/Engine.EdGraphNode withinpackageonly");
			UE_LOGF(LogVisualGraphUtils, Error, "Provide one or more path names for packages or objects to collect all tickables.");
			return;
		}

		TArray<FString> ObjectPathNames;
		for(const FString& InParam : InParams)
		{
			static const FString ObjectPathToken = TEXT("path"); 
			FString Token = ObjectPathToken;
			FString Content = InParam;

			if(InParam.Contains(TEXT("=")))
			{
				const int32 Pos = InParam.Find(TEXT("="));
				Token = InParam.Left(Pos);
				Content = InParam.Mid(Pos+1);
				Token.TrimStartAndEndInline();
				Token.ToLowerInline();
				Content.TrimStartAndEndInline();
			}

			if(Token == ObjectPathToken)
			{
				ObjectPathNames.Add(Content);
			}
		}

		TArray<UObject*> Objects;
		TArray<UObject*> OutersToUse;
		for(const FString& ObjectPathName : ObjectPathNames)
		{
			UE_CLOGF(FPackageName::IsShortPackageName(ObjectPathName), LogVisualGraphUtils, Warning, "Expected path name but got short name: \"%ls\"", *ObjectPathName);
			if(UObject* Object = FindObject<UObject>(nullptr, *ObjectPathName, EFindObjectFlags::None))
			{
				Objects.Add(Object);
			}
			else
			{
				UE_LOGF(LogVisualGraphUtils, Error, "Object with pathname '%ls' not found.", *ObjectPathName);
				return;
			}
		}

		const FString DotGraphContent = FVisualGraphObjectUtils::TraverseTickOrder(
			Objects).DumpDot();
		FPlatformApplicationMisc::ClipboardCopy(*DotGraphContent);

		UE_LOGF(LogVisualGraphUtils, Display, "The content has been copied to the clipboard.");
	})
);

#endif

///////////////////////////////////////////////////////////////////////////////////
/// FVisualGraphObjectUtilsReferenceCollector
///////////////////////////////////////////////////////////////////////////////////

class FVisualGraphObjectUtilsReferenceCollector : public FArchiveUObject
{
public:
	
	enum EReferenceKind
	{
		EReferenceKind_Direct,
		EReferenceKind_Outer
	};

	struct FReference
	{
		EReferenceKind Kind;
		UObject* Object;

		bool operator ==(const FReference& InReference) const
		{
			return InReference.Object == Object;
		}

		friend uint32 GetTypeHash(const FReference& InReference)
		{
			return GetTypeHash(InReference.Object);
		}
	};

private:

	// I/O function.  Called when an object reference is encountered.
	FArchive& operator<<( UObject*& Obj ) override
	{
		if( Obj )
		{
			FoundReference( Obj, EReferenceKind_Direct );
		}
		return *this;
	}

	virtual FArchive& operator<< (struct FSoftObjectPtr& Value) override
	{
		if ( Value.Get() )
		{
			FoundReference(Value.Get(), EReferenceKind_Direct);

			if(bCollectReferencesBySerialize)
			{
				Value.Get()->Serialize( *this );
			}
		}
		return *this;
	}
	
	virtual FArchive& operator<< (struct FSoftObjectPath& Value) override
	{
		if ( Value.ResolveObject() )
		{
			FoundReference(Value.ResolveObject(), EReferenceKind_Direct);

			if(bCollectReferencesBySerialize)
			{
				Value.ResolveObject()->Serialize( *this );
			}
		}
		return *this;
	}

	void FoundReference( UObject* Object, EReferenceKind Kind )
	{
		FReference Target;
		Target.Kind = Kind;
		Target.Object = Object;
		const TPair<UObject*, FReference> Reference(CurrentObject, Target);

		if(!ClassesToSkip.IsEmpty())
		{
			for(const UClass* ClassToSkip : ClassesToSkip)
			{
				if(Object->GetClass()->IsChildOf(ClassToSkip))
				{
					return;
				}
			}
		}

		if(!OutersToSkip.IsEmpty())
		{
			for(const UObject* OuterToSkip : OutersToSkip)
			{
				if(Object->IsInOuter(OuterToSkip))
				{
					return;
				}
			}
		}

		if(!OutersToUse.IsEmpty())
		{
			for(const UObject* OuterToUse : OutersToUse)
			{
				if(!Object->IsInOuter(OuterToUse))
				{
					return;
				}
			}
		}

		if ( RootSet.Find(Object) == nullptr )
		{
			if(bRecursive)
			{
				RootSetArray.Add( Object );
			}
			RootSet.Add(Object);
			References.Add(Reference);
		}
		else
		{
			References.FindOrAdd(Reference);
		}
	}

	UObject* CurrentObject;
	TSet<TPair<UObject*, FReference>>& References;
	TArray<UObject*> RootSetArray;
	TSet<UObject*> RootSet;
	const TArray<UClass*>& ClassesToSkip;
	const TArray<UObject*>& OutersToSkip;
	const TArray<UObject*>& OutersToUse;
	bool bTraverseObjectsInOuter;
	bool bCollectReferencesBySerialize;
	bool bRecursive;

public:

	FVisualGraphObjectUtilsReferenceCollector(
		TSet<UObject*> InRootSet,
		TSet<TPair<UObject*, FReference>>& InReferences,
		const TArray<UClass*>& InClassesToSkip,
		const TArray<UObject*>& InOutersToSkip,
		const TArray<UObject*>& InOutersToUse,
		bool InTraverseObjectsInOuter,
		bool InCollectReferencesBySerialize,
		bool InRecursive
		)
		: CurrentObject(nullptr)
		, References(InReferences)
		, RootSet(InRootSet)
		, ClassesToSkip(InClassesToSkip)
		, OutersToSkip(InOutersToSkip)
		, OutersToUse(InOutersToUse)
		, bTraverseObjectsInOuter(InTraverseObjectsInOuter)
		, bCollectReferencesBySerialize(InCollectReferencesBySerialize)
		, bRecursive(InRecursive)
	{
		ArIsObjectReferenceCollector = true;
		this->SetIsSaving(true);

		for ( UObject* Object : RootSet )
		{
			RootSetArray.Add( Object );
		}
		
		// loop through all the objects in the root set and serialize them
		for ( int RootIndex = 0; RootIndex < RootSetArray.Num(); ++RootIndex )
		{
			UObject* SourceObject = RootSetArray[RootIndex];

			// quick sanity check
			check(SourceObject);
			check(SourceObject->IsValidLowLevel());

			TGuardValue<UObject*> Guard(CurrentObject, SourceObject);

			if(bCollectReferencesBySerialize)
			{
				SourceObject->Serialize( *this );
			}

			if(bTraverseObjectsInOuter)
			{
				ForEachObjectWithOuter(SourceObject, [this](UObject* Inner)
				{
					FoundReference(Inner, EReferenceKind_Outer);
				}, EGetObjectsFlags::None);
			}
		}

	}

	virtual FString GetArchiveName() const override { return TEXT("FVisualGraphObjectUtilsReferenceCollector"); }
};

///////////////////////////////////////////////////////////////////////////////////
/// FVisualGraphObjectUtils
///////////////////////////////////////////////////////////////////////////////////

FVisualGraph FVisualGraphObjectUtils::TraverseUObjectReferences(
	const TArray<UObject*>& InObjects,
	const TArray<UClass*>& InClassesToSkip,
	const TArray<UObject*>& InOutersToSkip,
	const TArray<UObject*>& InOutersToUse,
	bool bTraverseObjectsInOuter,
	bool bCollectReferencesBySerialize,
	bool bRecursive)
{
	typedef FVisualGraphObjectUtilsReferenceCollector::FReference FReference;

	FVisualGraph Graph(TEXT("References"));
	if(InObjects.IsEmpty())
	{
		return Graph;
	}

	TSet<UObject*> RootObjects;
	for(UObject* InObject : InObjects)
	{
		RootObjects.Add(InObject);
	}

	struct Local
	{
		static int32 TraverseUObjectReferences(UObject* InObject, FVisualGraph& OutGraph)
		{
			const FName GuidName = *FString::Printf(TEXT("node_%d"), (int32)InObject->GetUniqueID());
			int32 NodeIndex = OutGraph.FindNode(GuidName);
			if(NodeIndex != INDEX_NONE)
			{
				return NodeIndex;
			}

			const FString PathName = InObject->GetPathName();
			if(PathName.StartsWith(TEXT("/Script/")))
			{
				return INDEX_NONE;
			}
			if(PathName.Contains(TEXT("TRASH_"), ESearchCase::CaseSensitive))
			{
				return INDEX_NONE;
			}

			NodeIndex = OutGraph.AddNode(GuidName, InObject->GetFName());

			OutGraph.GetNodes()[NodeIndex].SetTooltip(PathName);

			return NodeIndex;
		}
	};

	TSet<TPair<UObject*, FReference>> References;
	FVisualGraphObjectUtilsReferenceCollector Collector(
		RootObjects,
		References,
		InClassesToSkip,
		InOutersToSkip,
		InOutersToUse,
		bTraverseObjectsInOuter,
		bCollectReferencesBySerialize,
		bRecursive);
	
	for(const TPair<UObject*, FReference>& Reference : References)
	{
		UObject* Source = Reference.Key;
		const FReference& Target = Reference.Value;
		
		const int32 SourceNodeIndex = Local::TraverseUObjectReferences(Source, Graph);
		const int32 TargetNodeIndex = Local::TraverseUObjectReferences(Target.Object, Graph);

		if(SourceNodeIndex != INDEX_NONE && TargetNodeIndex != INDEX_NONE)
		{
			TOptional<EVisualGraphStyle> Style;
			
			if(Target.Kind == FVisualGraphObjectUtilsReferenceCollector::EReferenceKind_Outer)
			{
				Style = EVisualGraphStyle::Dashed;
			}
			
			Graph.AddEdge(
				SourceNodeIndex,
				TargetNodeIndex,
				EVisualGraphEdgeDirection::TargetToSource,
				NAME_None,
				TOptional<FName>(),
				TOptional<FLinearColor>(),
				Style);
		}
	}
	
	return Graph;
}

FVisualGraph FVisualGraphObjectUtils::TraverseTickOrder(const TArray<UObject*>& InObjects)
{
	typedef FVisualGraphObjectUtilsReferenceCollector::FReference FReference;

	FVisualGraph Graph(TEXT("References"));
	if(InObjects.IsEmpty())
	{
		return Graph;
	}

	TSet<UObject*> RootObjects;
	for(UObject* InObject : InObjects)
	{
		RootObjects.Add(InObject);
	}

	struct Local
	{
		static bool HasTickPrerequisite(
			UObject* InTargetObject,
			UObject* InSourceObject)
		{
			// we have found a potential edge - do we also have a tick order relationship?
			if(const AActor* TargetActor = Cast<AActor>(InTargetObject))
			{
				const TArray<struct FTickPrerequisite>& Prerequisites =
					TargetActor->PrimaryActorTick.GetPrerequisites();

				for(const struct FTickPrerequisite& Prerequisite : Prerequisites)
				{
					if(UObject* TickObject = Prerequisite.PrerequisiteObject.Get())
					{
						if(TickObject == InSourceObject)
						{
							return true;
						}
					}
				}

				if(const UActorComponent* SourceComponent = Cast<UActorComponent>(InSourceObject))
				{
					if(SourceComponent->GetOwner() == TargetActor)
					{
						return true;
					}
				}
			}

			if(const UActorComponent* TargetComponent = Cast<UActorComponent>(InTargetObject))
			{
				const TArray<struct FTickPrerequisite>& Prerequisites =
					TargetComponent->PrimaryComponentTick.GetPrerequisites();

				for(const struct FTickPrerequisite& Prerequisite : Prerequisites)
				{
					if(UObject* TickObject = Prerequisite.PrerequisiteObject.Get())
					{
						if(TickObject == InSourceObject)
						{
							return true;
						}
					}
				}
				
				if(const UActorComponent* SourceComponent = Cast<UActorComponent>(InSourceObject))
				{
					if(TargetComponent->IsInOuter(SourceComponent))
					{
						return true;
					}
				}
			}

			return false;
		}

		static void TraverseTickOrderEdges(
			const UObject* InTargetObject,
			const int32 InTargetNodeIndex,
			UObject* InTraversingObject,
			FVisualGraph& OutGraph,
			const TMap<UObject*, TArray<FReference>>& InTargetToSource,
			TMap<TPair<UObject*, UObject*>, bool>& OutVisitedEdges,
			TMap<UObject*, int32>& OutFarthestDistance,
			int32 InDistance
			)
		{
			check(InTargetNodeIndex != INDEX_NONE);
			
			const TArray<FReference>& Sources = InTargetToSource.FindChecked(InTargetObject);
			for(const FReference& Source : Sources)
			{
				TPair<UObject*, UObject*> Edge(Source.Object, InTraversingObject);
				if(OutVisitedEdges.Contains(Edge))
				{
					return;
				}
				OutVisitedEdges.Add(Edge, true);

				if(Source.Object == InTraversingObject)
				{
					return;
				}

				const FName GuidName = *FString::Printf(TEXT("node_%d"), (int32)Source.Object->GetUniqueID());
				const int32 SourceNodeIndex = OutGraph.FindNode(GuidName);
				if(SourceNodeIndex != INDEX_NONE)
				{
					
					TOptional<EVisualGraphStyle> Style;

					// we have found a potential edge - do we also have a tick order relationship?
					if(!HasTickPrerequisite(InTraversingObject, Source.Object) &&
						!HasTickPrerequisite(Source.Object, InTraversingObject))
					{
						Style = EVisualGraphStyle::Dotted;
					}
			
					OutGraph.AddEdge(
						SourceNodeIndex,
						InTargetNodeIndex,
						EVisualGraphEdgeDirection::TargetToSource,
						NAME_None,
						TOptional<FName>(),
						TOptional<FLinearColor>(),
						Style);
				}

				TraverseTickOrderEdges(Source.Object, InTargetNodeIndex, InTraversingObject, OutGraph, InTargetToSource, OutVisitedEdges, OutFarthestDistance, InDistance + 1);
			}
		}

		static int32 TraverseTickOrder(
			UObject* InObject,
			FVisualGraph& OutGraph,
			const TMap<UObject*, TArray<FReference>>& InTargetToSource,
			TMap<UObject*, bool>& OutVisitedObjects,
			TMap<TPair<UObject*, UObject*>, bool>& OutVisitedEdges,
			TMap<UObject*, int32>& OutFarthestDistance
			)
		{
			const FName GuidName = *FString::Printf(TEXT("node_%d"), (int32)InObject->GetUniqueID());
			int32 NodeIndex = OutGraph.FindNode(GuidName);
			if(NodeIndex != INDEX_NONE)
			{
				return NodeIndex;
			}

			if(OutVisitedObjects.Contains(InObject))
			{
				return INDEX_NONE;
			}
			OutVisitedObjects.Add(InObject, true);

			const FString PathName = InObject->GetPathName();
			if(PathName.StartsWith(TEXT("/Script/")))
			{
				return INDEX_NONE;
			}
			if(PathName.Contains(TEXT("TRASH_"), ESearchCase::CaseSensitive))
			{
				return INDEX_NONE;
			}

			bool bHasTick = false;
			if(const AActor* Actor = Cast<AActor>(InObject))
			{
				bHasTick = Actor->CanEverTick();
			}
			else if(const UActorComponent* ActorComponent = Cast<UActorComponent>(InObject))
			{
				bHasTick = ActorComponent->IsComponentTickEnabled();
			}

			if(bHasTick)
			{
				NodeIndex = OutGraph.AddNode(GuidName, InObject->GetFName());
			}

			// visit all targets. we do this to be sure that sources are processed prior to targets
			const TArray<FReference>& Sources = InTargetToSource.FindChecked(InObject);
			for(const FReference& Source : Sources)
			{
				TraverseTickOrder(Source.Object, OutGraph, InTargetToSource, OutVisitedObjects, OutVisitedEdges, OutFarthestDistance);
			}

			// for myself - walk back up and see if I am going to hit anything
			if(NodeIndex != INDEX_NONE)
			{
				OutGraph.GetNodes()[NodeIndex].SetTooltip(PathName);
				TraverseTickOrderEdges(InObject, NodeIndex, InObject, OutGraph, InTargetToSource, OutVisitedEdges, OutFarthestDistance, 0);
			}

			return NodeIndex;
		}
	};

	TSet<TPair<UObject*, FReference>> References;
	FVisualGraphObjectUtilsReferenceCollector Collector(
		RootObjects,
		References,
		TArray<UClass*>(),
		TArray<UObject*>(),
		TArray<UObject*>(),
		true,
		true,
		true);

	TMap<UObject*, TArray<FReference>> TargetToSource;

	for(const TPair<UObject*, FReference>& Reference : References)
	{
		TArray<FReference>& Sources = TargetToSource.FindOrAdd(Reference.Value.Object);
		FReference ReverseReference = Reference.Value;
		ReverseReference.Object = Reference.Key;
		Sources.Add(ReverseReference);
		TargetToSource.FindOrAdd(Reference.Key);
	}

	TMap<UObject*, bool> VisitedObjects;
	TMap<TPair<UObject*, UObject*>, bool> VisitedEdges;
	TMap<UObject*, int32> ObjectDistance;
	for(const TPair<UObject*, FReference>& Reference : References)
	{
		Local::TraverseTickOrder(Reference.Key, Graph, TargetToSource, VisitedObjects, VisitedEdges, ObjectDistance);
	}

	Graph.TransitiveReduction([](FVisualGraphEdge& Edge) -> bool
	{
		Edge.SetColor(FLinearColor::Gray);
		return true /* keep edge */;
	});
	
	return Graph;
}



