// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "VisualGraphEdge.h"
#include "VisualGraphElement.h"
#include "VisualGraphNode.h"

#define UE_API VISUALGRAPHUTILS_API

class FVisualGraph;

class FVisualGraphSubGraph : public FVisualGraphElement
{
public:

	FVisualGraphSubGraph()
	: FVisualGraphElement()
	, ParentGraphIndex(INDEX_NONE)
	{}

	virtual ~FVisualGraphSubGraph() override = default;

	int32 GetParentGraphIndex() const { return ParentGraphIndex; }
	const TArray<int32>& GetNodes() const { return Nodes; }

protected:

	UE_API virtual FString DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const override;

	int32 ParentGraphIndex;
	TArray<int32> Nodes;

	friend class FVisualGraph;
};

class FVisualGraph : public FVisualGraphElement
{
public:

	FVisualGraph()
	: FVisualGraphElement()
	{}

	virtual ~FVisualGraph() override = default;

	UE_API FVisualGraph(const FName& InName, const FName& InDisplayName = NAME_None);

	const TArray<FVisualGraphNode>& GetNodes() const { return Nodes; }
	const TArray<FVisualGraphEdge>& GetEdges() const { return Edges; }
	const TArray<FVisualGraphSubGraph>& GetSubGraphs() const { return SubGraphs; }

	UE_API int32 AddNode(
		const FName& InName,
		TOptional<FName> InDisplayName = TOptional<FName>(),
		TOptional<FLinearColor> InColor = TOptional<FLinearColor>(),
		TOptional<EVisualGraphShape> InShape = TOptional<EVisualGraphShape>(), 
		TOptional<EVisualGraphStyle> InStyle = TOptional<EVisualGraphStyle>());

	UE_API int32 AddEdge(
		int32 InSourceNode,
		int32 InTargetNode,
		EVisualGraphEdgeDirection InDirection,
		const FName& InName = NAME_None,
		TOptional<FName> InDisplayName = TOptional<FName>(),
		TOptional<FLinearColor> InColor = TOptional<FLinearColor>(),
		TOptional<EVisualGraphStyle> InStyle = TOptional<EVisualGraphStyle>());

	UE_API int32 AddSubGraph(
		const FName& InName,
		TOptional<FName> InDisplayName = TOptional<FName>(),
		int32 InParentGraphIndex = INDEX_NONE,
		TOptional<FLinearColor> InColor = TOptional<FLinearColor>(),
		TOptional<EVisualGraphStyle> InStyle = TOptional<EVisualGraphStyle>(),
		const TArray<int32> InNodes = TArray<int32>());

	UE_API int32 FindNode(const FName& InName) const;
	UE_API int32 FindEdge(const FName& InName) const;
	UE_API int32 FindSubGraph(const FName& InName) const;

	UE_API bool AddNodeToSubGraph(int32 InNodeIndex, int32 InSubGraphIndex);
	UE_API bool RemoveNodeFromSubGraph(int32 InNodeIndex);

	UE_API void TransitiveReduction(TFunction<bool(FVisualGraphEdge&)> KeepEdgeFunction);

	FORCEINLINE FString DumpDot() const
	{
		return DumpDot(this, 0);
	}

protected:

	UE_API virtual FString DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const override;

	template<typename T>
	FORCEINLINE static void RefreshNameMap(const TArray<T>& InElements, TMap<FName, int32>& OutMap)
	{
		OutMap.Reset();
		for(const T& Element: InElements)
		{
			OutMap.Add(Element.Name, Element.Index);
		}
	}

	template<typename T>
	FORCEINLINE static void RefreshNameMapIfNeeded(const TArray<T>& InElements, TMap<FName, int32>& OutMap)
	{
		if(OutMap.Num() == InElements.Num())
		{
			return;
		}
		RefreshNameMap(InElements, OutMap);
	}

	template<typename T>
	FORCEINLINE static int32 AddElement(const T& InElement, TArray<T>& OutElements, TMap<FName, int32>& OutMap)
	{
		const int32 AddedIndex = OutElements.Add(InElement);
		if(OutElements.IsValidIndex(AddedIndex))
		{
			OutElements[AddedIndex].Index = AddedIndex;
			OutMap.Add(InElement.Name, AddedIndex);
		}
		return AddedIndex;
	}

	static UE_API bool IsNameAvailable(const FName& InName, const TMap<FName, int32>& InMap);
	static UE_API FName GetUniqueName(const FName& InName, const TMap<FName, int32>& InMap);

	TArray<FVisualGraphNode> Nodes;
	TArray<FVisualGraphEdge> Edges;
	TArray<FVisualGraphSubGraph> SubGraphs;

	TMap<FName,int32> NodeNameMap;
	TMap<FName,int32> EdgeNameMap;
	TMap<FName,int32> SubGraphNameMap;
};

#undef UE_API
