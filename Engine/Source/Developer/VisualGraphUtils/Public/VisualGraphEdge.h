// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "VisualGraphElement.h"

#include "VisualGraphEdge.generated.h"

#define UE_API VISUALGRAPHUTILS_API

class FVisualGraph;

UENUM()
enum class EVisualGraphEdgeDirection : uint8
{
	SourceToTarget,
	TargetToSource,
	BothWays
};

class FVisualGraphEdge : public FVisualGraphElement
{
public:

	FVisualGraphEdge()
	: FVisualGraphElement()
	, Direction(EVisualGraphEdgeDirection::SourceToTarget)
	{}

	virtual ~FVisualGraphEdge() override = default;

	int32 GetSourceNode() const { return SourceNode; }
	int32 GetTargetNode() const { return TargetNode; }
	EVisualGraphEdgeDirection GetDirection() const { return Direction; }

protected:

	UE_API virtual FString DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const override;

	int32 SourceNode;
	int32 TargetNode;
	EVisualGraphEdgeDirection Direction;

	friend class FVisualGraph;
	friend class FVisualGraphSubGraph;
};

#undef UE_API
