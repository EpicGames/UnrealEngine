// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"

#include "VisualGraphElement.generated.h"

#define UE_API VISUALGRAPHUTILS_API

class FVisualGraph;

UENUM()
enum class EVisualGraphShape : uint8
{
	Box,
	Polygon,
	Ellipse,
	Circle,
	Triangle,
	PlainText,
	Diamond,
	Parallelogram,
	House
};

UENUM()
enum class EVisualGraphStyle : uint8
{
	Filled,
	Diagonals,
	Rounded,
	Dashed,
	Dotted,
	Solid,
	Bold
};

class FVisualGraphElement
{
public:

	FVisualGraphElement()
	: Name(NAME_None)
	, DisplayName()
	, Index(INDEX_NONE)
	, Color()
	, Style()
	{}

	virtual ~FVisualGraphElement() = default;

	FName GetName() const { return Name; }
	FName GetDisplayName() const { return DisplayName.IsSet() ? DisplayName.GetValue() : Name; }
	TOptional<FString> GetTooltip() const { return Tooltip; }
	void SetTooltip(const FString& InTooltip) const { Tooltip = InTooltip; }
	int32 GetIndex() const { return Index; }
	TOptional<FLinearColor> GetColor() const { return Color; }
	void SetColor(FLinearColor InValue) { Color = InValue; }
	TOptional<EVisualGraphStyle> GetStyle() const { return Style; }
	void SetStyle(EVisualGraphStyle InValue) { Style = InValue; }

protected:

	virtual FString DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const = 0;
	static UE_API FString DumpDotIndentation(int32 InIndentation);
	static UE_API FString DumpDotColor(const TOptional<FLinearColor>& InColor);
	static UE_API FString DumpDotShape(const TOptional<EVisualGraphShape>& InShape);
	static UE_API FString DumpDotStyle(const TOptional<EVisualGraphStyle>& InStyle);

	FName Name;
	TOptional<FName> DisplayName;
	mutable TOptional<FString> Tooltip;
	int32 Index;
	TOptional<FLinearColor> Color;
	TOptional<EVisualGraphStyle> Style;
	
	friend class FVisualGraph;
};

#undef UE_API
