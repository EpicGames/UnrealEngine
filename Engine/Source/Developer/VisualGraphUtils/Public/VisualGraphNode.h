// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "VisualGraphElement.h"

#define UE_API VISUALGRAPHUTILS_API

class FVisualGraph;

class FVisualGraphNode : public FVisualGraphElement
{
public:

	FVisualGraphNode()
	: FVisualGraphElement()
	, Shape()
	, SubGraphIndex(INDEX_NONE)
	{}

	virtual ~FVisualGraphNode() override = default;

	TOptional<EVisualGraphShape> GetShape() const { return Shape; }
	void SetShape(EVisualGraphShape InValue) { Shape = InValue; }

protected:

	UE_API virtual FString DumpDot(const FVisualGraph* InGraph, int32 InIndendation) const override;

	TOptional<EVisualGraphShape> Shape;
	int32 SubGraphIndex;

	friend class FVisualGraph;
	friend class FVisualGraphSubGraph;
};

#undef UE_API
