// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "VisualGraph.h"

#define UE_API VISUALGRAPHUTILS_API

class UClass;
class UObject;

class FVisualGraphObjectUtils
{
public:

	static UE_API FVisualGraph TraverseUObjectReferences(
		const TArray<UObject*>& InObjects,
		const TArray<UClass*>& InClassesToSkip = TArray<UClass*>(),
		const TArray<UObject*>& InOutersToSkip = TArray<UObject*>(),
		const TArray<UObject*>& InOutersToUse = TArray<UObject*>(),
		bool bTraverseObjectsInOuter = true,
		bool bCollectReferencesBySerialize = true,
		bool bRecursive = true
		);

	static UE_API FVisualGraph TraverseTickOrder(
		const TArray<UObject*>& InObjects
		);
};

#undef UE_API
