// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VisualGraphUtils : ModuleRules
{
    public VisualGraphUtils(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "Engine",
            }
        );
        
        if (Target.bBuildEditor == true)
        {
	        PublicDependencyModuleNames.AddRange(
		        new string[]
		        {
			        "ApplicationCore",
		        }
	        );
        }        
    }
}
